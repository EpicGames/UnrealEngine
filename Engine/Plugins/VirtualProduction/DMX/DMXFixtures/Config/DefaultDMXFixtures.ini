[CoreRedirects]

; 4.27
+PropertyRedirects=(OldName="DMXFixtureComponent.IsEnabled",NewName="DMXFixtureComponent.bIsEnabled")
+PropertyRedirects=(OldName="DMXFixtureComponent.UsingMatrixData",NewName="DMXFixtureComponent.bUsingMatrixData")
+PropertyRedirects=(OldName="DMXFixtureComponent.UseInterpolation",NewName="DMXFixtureComponent.bUseInterpolation")

+PropertyRedirects=(OldName="DMXFixtureComponentColor.ChannelName1",NewName="DMXComponent.DMXChannel1")
+PropertyRedirects=(OldName="DMXFixtureComponentColor.ChannelName2",NewName="DMXComponent.DMXChannel2")
+PropertyRedirects=(OldName="DMXFixtureComponentColor.ChannelName3",NewName="DMXComponent.DMXChannel3")
+PropertyRedirects=(OldName="DMXFixtureComponentColor.ChannelName4",NewName="DMXComponent.DMXChannel4")

+FunctionRedirects=(OldName="DMXFixtureComponentSingle.DMXInterpolatedStep",NewName="DMXFixtureComponentSingle.GetDMXInterpolatedStep")
+FunctionRedirects=(OldName="DMXFixtureComponentSingle.DMXInterpolatedValue",NewName="DMXFixtureComponentSingle.GetDMXInterpolatedValue")
+FunctionRedirects=(OldName="DMXFixtureComponentSingle.DMXTargetValue",NewName="DMXFixtureComponentSingle.GetDMXTargetValue")
+FunctionRedirects=(OldName="DMXFixtureComponentSingle.DMXIsInterpolationDone",NewName="DMXFixtureComponentSingle.IsDMXInterpolationDone")
+FunctionRedirects=(OldName="DMXFixtureComponentSingle.SetComponent",NewName="DMXFixtureComponentSingle.SetValueNoInterp")

+FunctionRedirects=(OldName="DMXFixtureComponentDouble.DMXInterpolatedStep",NewName="DMXFixtureComponentDouble.GetDMXInterpolatedStep")
+FunctionRedirects=(OldName="DMXFixtureComponentDouble.DMXInterpolatedValue",NewName="DMXFixtureComponentDouble.GetDMXInterpolatedValue")
+FunctionRedirects=(OldName="DMXFixtureComponentDouble.DMXTargetValue",NewName="DMXFixtureComponentDouble.GetDMXTargetValue")
+FunctionRedirects=(OldName="DMXFixtureComponentDouble.DMXIsInterpolationDone",NewName="DMXFixtureComponentDouble.IsDMXInterpolationDone")
+FunctionRedirects=(OldName="DMXFixtureComponentDouble.SetComponentChannel1",NewName="DMXFixtureComponentDouble.SetChannel1ValueNoInterp")
+FunctionRedirects=(OldName="DMXFixtureComponentDouble.SetComponentChannel2",NewName="DMXFixtureComponentDouble.SetChannel2ValueNoInterp")

+FunctionRedirects=(OldName="DMXFixtureComponentColor.SetComponent",NewName="DMXFixtureComponentColor.SetColorNoInterp")

