{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DMX Fixtures",
	"Description": "DMX Light Fixtures Blueprints",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Modules": [
		{
			"Name": "DMXFixtures",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DMXEngine",
			"Enabled": true
		},
		{
			"Name": "DMXModularFeatures",
			"Enabled": true
		},
		{
			"Name": "DMXProtocol",
			"Enabled": true
		},
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		}
	],
	"IsExperimentalVersion": false,
	"Installed": false
}