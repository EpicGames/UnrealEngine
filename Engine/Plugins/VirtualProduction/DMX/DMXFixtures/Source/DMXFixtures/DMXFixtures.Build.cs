// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXFixtures : ModuleRules
{
	public DMXFixtures(ReadOnlyTargetRules Target) : base(Target)
	{
	   PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

       PublicDependencyModuleNames.AddRange(new string[] { 
		   "Core",
		   "CoreUObject",
			"DMXFixtureActorInterface",
		   "DMXRuntime", 
		   "DMXProtocol",
		   "Engine",
		   "RenderCore", 
		   "RHI", 
		   "ProceduralMeshComponent" });

       PrivateDependencyModuleNames.AddRange(new string[] {  });

	}
}
