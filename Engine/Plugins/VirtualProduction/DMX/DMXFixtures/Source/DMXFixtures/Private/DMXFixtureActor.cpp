// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureActor.h"

#include "Components/ArrowComponent.h"
#include "Components/SpotLightComponent.h"
#include "Components/StaticMeshComponent.h"
#include "DMXFixtureComponent.h"
#include "DMXStats.h"
#include "Materials/MaterialInstanceDynamic.h"


DECLARE_CYCLE_STAT(TEXT("FixtureActor Push Normalized Values"), STAT_FixtureActorPushNormalizedValuesPerAttribute, STATGROUP_DMX);

ADMXFixtureActor::ADMXFixtureActor()
{
	Base = CreateDefaultSubobject<USceneComponent>(TEXT("Base"));
	RootComponent = Base;

	Yoke = CreateDefaultSubobject<USceneComponent>(TEXT("Yoke"));
	Yoke->SetupAttachment(Base);

	Head = CreateDefaultSubobject<USceneComponent>(TEXT("Head"));
	Head->SetupAttachment(Yoke);

	PointLight = CreateDefaultSubobject<UPointLightComponent>(TEXT("Fixture PointLight"));
	PointLight->SetupAttachment(Head);
	PointLight->SetCastShadows(false);
	PointLight->bAffectsWorld = false;

	SpotLight = CreateDefaultSubobject<USpotLightComponent>(TEXT("Fixture SpotLight"));
	SpotLight->SetupAttachment(Head);
	SpotLight->SetCastShadows(false);
	SpotLight->bAffectsWorld = true;

	OcclusionDirection = CreateDefaultSubobject<UArrowComponent>(TEXT("Occlusion"));
	OcclusionDirection->SetupAttachment(Head);

	// set default values
	LightIntensityMax = 2000;
	LightDistanceMax = 1000;
	LightColorTemp = 6500;
	SpotlightIntensityScale = 1.0f;
	PointlightIntensityScale = 1.0f;
	LightCastShadow = false;
	UseDynamicOcclusion = false;
	LensRadius = 10.0f;
	QualityLevel = EDMXFixtureQualityLevel::HighQuality;
	ZoomQuality = 1.0f;
	BeamQuality = 1.0f;
	DisableLights = false;
	HasBeenInitialized = false;
}

void ADMXFixtureActor::OnMVRGetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames, TArray<FName>& OutMatrixAttributeNames) const
{
	for (UDMXFixtureComponent* DMXFixtureComponent : TInlineComponentArray<UDMXFixtureComponent*>(this))
	{
		TArray<FName> SupportedAttributeNamesOfComponent;
		DMXFixtureComponent->GetSupportedDMXAttributes(SupportedAttributeNamesOfComponent);

		OutAttributeNames.Append(SupportedAttributeNamesOfComponent);
	}
}

#if WITH_EDITOR
void ADMXFixtureActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		FeedFixtureData();
	}
}
#endif

void ADMXFixtureActor::InitializeFixture(UStaticMeshComponent* StaticMeshLens, UStaticMeshComponent* StaticMeshBeam)
{
	GetComponents(StaticMeshComponents);

	// Create dynamic materials
	DynamicMaterialLens = UMaterialInstanceDynamic::Create(LensMaterialInstance, nullptr);
	DynamicMaterialBeam = UMaterialInstanceDynamic::Create(BeamMaterialInstance, nullptr);
	DynamicMaterialSpotLight = UMaterialInstanceDynamic::Create(SpotLightMaterialInstance, nullptr);
	DynamicMaterialPointLight = UMaterialInstanceDynamic::Create(PointLightMaterialInstance, nullptr);

	// Get lens width (support scaling)
	if (StaticMeshLens)
	{
		//FBoxSphereBounds LocalBounds = StaticMeshLens->CalcLocalBounds();
		//FVector Scale = StaticMeshLens->GetRelativeScale3D();
		//float BiggestComponentScale = Scale.GetMax();
		//LensRadius = LocalBounds.SphereRadius * BiggestComponentScale * 0.97f;
		FBoxSphereBounds Bounds = StaticMeshLens->Bounds;
		LensRadius = Bounds.SphereRadius * 0.9f;
	}

	// Feed fixture data into materials and lights
	FeedFixtureData();

	// Assign dynamic materials to static meshes
	if (StaticMeshLens)
	{
		StaticMeshLens->SetMaterial(0, DynamicMaterialLens);
	}

	// Make sure beam doesnt have any scale applied or it wont render correctly
	if (StaticMeshBeam)
	{
		StaticMeshBeam->SetMaterial(0, DynamicMaterialBeam);
		StaticMeshBeam->SetWorldScale3D(FVector(1, 1, 1));
	}

	// Assign dynamic materials to lights
	SpotLight->SetMaterial(0, DynamicMaterialSpotLight);
	PointLight->SetMaterial(0, DynamicMaterialPointLight);

	// Initialize fixture components
	for (UDMXFixtureComponent* DMXComponent : TInlineComponentArray<UDMXFixtureComponent*>(this))
	{
		DMXComponent->Initialize();
	}

	HasBeenInitialized = true;
}

void ADMXFixtureActor::UpdateSpotLightIntensity()
{
	const float SterdianRatio = UE_PI * 2.f * (1.f - SpotLight->GetCosHalfConeAngle());

	SpotLight->SetIntensity(LightIntensityMax * SpotlightIntensityScale * (1.f / SterdianRatio));
}

void ADMXFixtureActor::FeedFixtureData()
{
	// BeamQuality and ZoomQuality modulate the "stepSize" for the raymarch beam shader
	// lower value is visually better
	switch (QualityLevel)
	{
		case(EDMXFixtureQualityLevel::LowQuality): 
			ZoomQuality = 1.0f;
			BeamQuality = 4.0f;
			break;
		case(EDMXFixtureQualityLevel::MediumQuality): 
			ZoomQuality = 1.0f;
			BeamQuality = 2.0f;
			break;
		case(EDMXFixtureQualityLevel::HighQuality): 
			ZoomQuality = 1.0f;
			BeamQuality = 1.0f;
			break;
		case(EDMXFixtureQualityLevel::UltraQuality): 
			ZoomQuality = 2.0f;
			BeamQuality = 0.33f;
			break;
	}
	
	// Clamp values, also clamps UI
	if (QualityLevel == EDMXFixtureQualityLevel::Custom)
	{
		ZoomQuality = FMath::Clamp(ZoomQuality, 1.0f, 4.0f);
		BeamQuality = FMath::Clamp(BeamQuality, 0.2f, 4.0f);
	}

	if (DynamicMaterialBeam)
	{
		DynamicMaterialBeam->SetScalarParameterValue("DMX Quality Level", BeamQuality);
		DynamicMaterialBeam->SetScalarParameterValue("DMX Max Light Distance", LightDistanceMax);
		DynamicMaterialBeam->SetScalarParameterValue("DMX Max Light Intensity", LightIntensityMax * SpotlightIntensityScale);
	}

	if (DynamicMaterialLens)
	{
		DynamicMaterialLens->SetScalarParameterValue("DMX Max Light Intensity", LightIntensityMax * SpotlightIntensityScale);
		DynamicMaterialBeam->SetScalarParameterValue("DMX Lens Radius", LensRadius);
	}

	// Set lights
	UpdateSpotLightIntensity();
	SpotLight->SetTemperature(LightColorTemp);
	SpotLight->SetCastShadows(LightCastShadow);
	SpotLight->SetAttenuationRadius(LightDistanceMax);

	PointLight->SetIntensity(LightIntensityMax * PointlightIntensityScale);
	PointLight->SetTemperature(LightColorTemp);
	PointLight->SetCastShadows(LightCastShadow);
	PointLight->SetAttenuationRadius(LightDistanceMax);
}

void ADMXFixtureActor::SetLightIntensityMax(float NewLightIntensityMax)
{
	LightIntensityMax = NewLightIntensityMax;

	if (DynamicMaterialBeam)
	{
		DynamicMaterialBeam->SetScalarParameterValue("DMX Max Light Intensity", LightIntensityMax * SpotlightIntensityScale);
	}

	if (DynamicMaterialLens)
	{
		DynamicMaterialLens->SetScalarParameterValue("DMX Max Light Intensity", LightIntensityMax * SpotlightIntensityScale);
	}

	SpotLight->SetIntensity(LightIntensityMax * SpotlightIntensityScale);
	PointLight->SetIntensity(LightIntensityMax * PointlightIntensityScale);
}

void ADMXFixtureActor::SetLightDistanceMax(float NewLightDistanceMax)
{
	LightDistanceMax = NewLightDistanceMax;

	if (DynamicMaterialBeam)
	{
		DynamicMaterialBeam->SetScalarParameterValue("DMX Max Light Distance", LightDistanceMax);
	}

	SpotLight->SetAttenuationRadius(LightDistanceMax);
	PointLight->SetAttenuationRadius(LightDistanceMax);
}

void ADMXFixtureActor::SetLightColorTemp(float NewLightColorTemp)
{
	LightColorTemp = NewLightColorTemp;

	SpotLight->SetTemperature(LightColorTemp);
	PointLight->SetTemperature(LightColorTemp);
}

void ADMXFixtureActor::SetSpotlightIntensityScale(float NewSpotlightIntensityScale)
{
	SpotlightIntensityScale = NewSpotlightIntensityScale;

	if (DynamicMaterialBeam)
	{
		DynamicMaterialBeam->SetScalarParameterValue("DMX Max Light Intensity", LightIntensityMax * SpotlightIntensityScale);
	}

	if (DynamicMaterialLens)
	{
		DynamicMaterialLens->SetScalarParameterValue("DMX Max Light Intensity", LightIntensityMax * SpotlightIntensityScale);
	}

	UpdateSpotLightIntensity();
}

void ADMXFixtureActor::SetPointlightIntensityScale(float NewPointlightIntensityScale)
{
	PointlightIntensityScale = NewPointlightIntensityScale;

	PointLight->SetIntensity(LightIntensityMax * PointlightIntensityScale);
}

void ADMXFixtureActor::SetLightCastShadow(bool bLightShouldCastShadow)
{
	LightCastShadow = bLightShouldCastShadow;

	SpotLight->SetCastShadows(bLightShouldCastShadow);
	PointLight->SetCastShadows(bLightShouldCastShadow);
}

void ADMXFixtureActor::PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttribute)
{
	SCOPE_CYCLE_COUNTER(STAT_FixtureActorPushNormalizedValuesPerAttribute);

	if (HasBeenInitialized)
	{
		Super::PushNormalizedValuesPerAttribute(ValuePerAttribute);
	}
}
