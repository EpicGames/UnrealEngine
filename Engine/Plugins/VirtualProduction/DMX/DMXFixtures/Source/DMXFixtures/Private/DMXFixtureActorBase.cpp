// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureActorBase.h"

#include "DMXFixtureComponent.h"
#include "DMXInterpolation.h"
#include "DMXStats.h"
#include "Game/DMXComponent.h"



DECLARE_CYCLE_STAT(TEXT("BaseFixtureActor Push Normalized Values"), STAT_BaseFixtureActorPushNormalizedValuesPerAttribute, STATGROUP_DMX);

ADMXFixtureActorBase::ADMXFixtureActorBase()
{
	PrimaryActorTick.bCanEverTick = true;

#if WITH_EDITORONLY_DATA
	bRunConstructionScriptOnDrag = false;
#endif

	DMX = CreateDefaultSubobject<UDMXComponent>(TEXT("DMX"));
}

void ADMXFixtureActorBase::InterpolateDMXComponents(float DeltaSeconds)
{
	// Get current components (supports PIE)
	TInlineComponentArray<UDMXFixtureComponent*> DMXComponents;
	GetComponents<UDMXFixtureComponent>(DMXComponents);

	for (auto& DMXComponent : DMXComponents)
	{
		if (DMXComponent->bIsEnabled && DMXComponent->bUseInterpolation && DMXComponent->IsRegistered())
		{
			for (auto& Cell : DMXComponent->Cells)
			{
				DMXComponent->CurrentCell = &Cell;
				for (auto& ChannelInterp : Cell.ChannelInterpolation)
				{
					if (ChannelInterp.IsUpdating)
					{
						// Update
						ChannelInterp.Travel(DeltaSeconds);

						// Run BP event
						DMXComponent->InterpolateComponent(DeltaSeconds);

						// Is done?
						if (ChannelInterp.IsInterpolationDone())
						{
							ChannelInterp.EndInterpolation();
						}
					}
				}
			}
		}
	}
}

void ADMXFixtureActorBase::PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttribute)
{
	SCOPE_CYCLE_COUNTER(STAT_BaseFixtureActorPushNormalizedValuesPerAttribute);

	for (UDMXFixtureComponent* DMXComponent : TInlineComponentArray<UDMXFixtureComponent*>(this))
	{
		if (DMXComponent->bIsEnabled)
		{
			DMXComponent->PushNormalizedValuesPerAttribute(ValuePerAttribute);
		}
	}
}
