// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureActorMatrix.h"

#include "Components/SpotLightComponent.h"
#include "DMXStats.h"
#include "DMXFixtureComponentColor.h"
#include "Engine/World.h"
#include "DMXFixtureComponentDouble.h"
#include "Game/DMXComponent.h"

#include "DMXFixtureComponentSingle.h"
#include "Engine/Texture2D.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Rendering/Texture2DResource.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "RenderingThread.h"
#include "ProceduralMeshComponent.h"

DECLARE_CYCLE_STAT(TEXT("Fixture Actor Matrix Push Fixture Matrix Cell Data"), STAT_FixtureActorMatrixPushFixtureMatrixCellData, STATGROUP_DMX);

namespace
{
	void UpdateMatrixTexture(uint8* MatrixData, UTexture2D* DynamicTexture, int32 MipIndex, uint32 NumRegions, const FUpdateTextureRegion2D& Region, uint32 SrcPitch, uint32 SrcBpp)
	{
		if (DynamicTexture->GetResource())
		{
			ENQUEUE_RENDER_COMMAND(UpdateTextureRegionsData)(
				[=](FRHICommandListImmediate& RHICmdList)
				{
					FTexture2DResource* Resource = (FTexture2DResource*)DynamicTexture->GetResource();
					RHICmdList.UpdateTexture2D(
						Resource->GetTexture2DRHI(),
						MipIndex,
						Region,
						SrcPitch,
						MatrixData
						+ Region.SrcY * SrcPitch
						+ Region.SrcX * SrcBpp
					);

				});

		}
	}
}

ADMXFixtureActorMatrix::ADMXFixtureActorMatrix()
{
	MatrixHead = CreateDefaultSubobject<UProceduralMeshComponent>(TEXT("MatrixHead"));
	MatrixHead->SetupAttachment(Head);

	SpotLight->SetInnerConeAngle(65);
	SpotLight->SetOuterConeAngle(80);

	MatrixHeight = 100;
	MatrixWidth = 100;
	MatrixDepth = 10;

	NbrTextureRows = 1;
	XCells = 1;
	YCells = 1;
	MatrixDataSize = 0;
}

#if WITH_EDITOR
void ADMXFixtureActorMatrix::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	FeedFixtureData();
}
#endif

void ADMXFixtureActorMatrix::OnMVRGetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames, TArray<FName>& OutMatrixAttributeNames) const
{
	Super::OnMVRGetSupportedDMXAttributes_Implementation(OutAttributeNames, OutMatrixAttributeNames);
	
	for (UDMXFixtureComponent* DMXComponent : TInlineComponentArray<UDMXFixtureComponent*>(this))
	{
		if (UDMXFixtureComponentColor* ColorComponent = Cast<UDMXFixtureComponentColor>(DMXComponent))
		{
			OutMatrixAttributeNames.Add(ColorComponent->DMXChannel1.Name);
			OutMatrixAttributeNames.Add(ColorComponent->DMXChannel2.Name);
			OutMatrixAttributeNames.Add(ColorComponent->DMXChannel3.Name);
			OutMatrixAttributeNames.Add(ColorComponent->DMXChannel4.Name);
		}
		else if (UDMXFixtureComponentSingle* SingleComponent = Cast<UDMXFixtureComponentSingle>(DMXComponent))
		{
			// Single channel component - hardcoded and limited to specific names, as on PushFixtureMatrixCellData
			if (SingleComponent->DMXChannel.Name.Name == FName("Dimmer"))
			{
				OutMatrixAttributeNames.Add(FName("Dimmer"));
			}
			else if (SingleComponent->DMXChannel.Name.Name == FName("Pan"))
			{
				OutMatrixAttributeNames.Add(FName("Pan"));
			}
			else if (SingleComponent->DMXChannel.Name.Name == FName("Tilt"))
			{
				OutMatrixAttributeNames.Add(FName("Tilt"));
			}
		}
	}
}

void ADMXFixtureActorMatrix::InitializeMatrixFixture()
{
	UDMXEntityFixturePatch* FixturePatch = DMX->GetFixturePatch();
	if (FixturePatch)
	{
		GetComponents(StaticMeshComponents);

		// Create dynamic materials
		DynamicMaterialLens = UMaterialInstanceDynamic::Create(LensMaterialInstance, nullptr);
		DynamicMaterialBeam = UMaterialInstanceDynamic::Create(BeamMaterialInstance, nullptr);
		DynamicMaterialSpotLight = UMaterialInstanceDynamic::Create(SpotLightMaterialInstance, nullptr);
		DynamicMaterialPointLight = UMaterialInstanceDynamic::Create(PointLightMaterialInstance, nullptr);

		float Quality = 1.0f;
		switch (QualityLevel)
		{
			case(EDMXFixtureQualityLevel::LowQuality): Quality = 0.25f; break;
			case(EDMXFixtureQualityLevel::MediumQuality): Quality = 0.5f; break;
			case(EDMXFixtureQualityLevel::HighQuality): Quality = 1.0f; break;
			case(EDMXFixtureQualityLevel::UltraQuality): Quality = 2.0f; break;
			default: Quality = 1.0f;
		}

		// Get matrix properties
		FDMXFixtureMatrix MatrixProperties;
		FixturePatch->GetMatrixProperties(MatrixProperties);
		XCells = MatrixProperties.XCells;
		YCells = MatrixProperties.YCells;

		// Limit cells [1-DMX_UNIVERSE_SIZE]
		constexpr int DMXUniverseSize = DMX_UNIVERSE_SIZE;

		XCells = FMath::Max(XCells, 1);
		YCells = FMath::Max(YCells, 1);
		XCells = FMath::Min(XCells, DMXUniverseSize);
		YCells = FMath::Min(YCells, DMXUniverseSize);

		int NbrCells = XCells * YCells;

		// Create array to hold data in bgra order
		NbrTextureRows = 2;	// using 2 rows to store dmx data
		MatrixDataSize = NbrCells * 4 * NbrTextureRows;

		MatrixData.Reset(MatrixDataSize);
		MatrixData.AddZeroed(MatrixDataSize);
		for (int i = 0; i < MatrixDataSize; i++)
		{
			MatrixData[i] = 128;
		}

		// Generate runtime procedural mesh
		GenerateMatrixMesh();

		// Create transient texture at runtime (DynamicTexture)
		int TextureWidth = NbrCells;
		int TextureHeight = NbrTextureRows;
		MatrixDataTexture = UTexture2D::CreateTransient(TextureWidth, TextureHeight, EPixelFormat::PF_B8G8R8A8);
		MatrixDataTexture->SRGB = 0;
		MatrixDataTexture->bNoTiling = true;
		MatrixDataTexture->Filter = TextureFilter::TF_Nearest; //pixelated
		MatrixDataTexture->AddressX = TextureAddress::TA_Clamp;
		MatrixDataTexture->AddressY = TextureAddress::TA_Clamp;
		MatrixDataTexture->CompressionSettings = TextureCompressionSettings::TC_VectorDisplacementmap;
		MatrixDataTexture->UpdateResource(); //to initialize resource

		TextureRegion = FUpdateTextureRegion2D(0, 0, 0, 0, TextureWidth, TextureHeight);

		// Push fixture data into materials and lights
		FeedFixtureData();

		// Assign dynamic materials to lights
		SpotLight->SetMaterial(0, DynamicMaterialSpotLight);
		PointLight->SetMaterial(0, DynamicMaterialPointLight);

		// feed matrix properties to lens material
		if (DynamicMaterialLens)
		{
			DynamicMaterialLens->SetScalarParameterValue("XCells", XCells);
			DynamicMaterialLens->SetScalarParameterValue("YCells", YCells);
			DynamicMaterialLens->SetScalarParameterValue("CellWidth", MatrixWidth / XCells);
			DynamicMaterialLens->SetScalarParameterValue("CellHeight", MatrixHeight / YCells);
			DynamicMaterialLens->SetTextureParameterValue("MatrixData", MatrixDataTexture);
			MatrixHead->SetMaterial(0, DynamicMaterialLens);
		}

		// feed matrix properties to beam material
		if (DynamicMaterialBeam)
		{
			int NbrSamples = FMath::CeilToInt(Quality * 4);
			DynamicMaterialBeam->SetScalarParameterValue("NbrSamples", NbrSamples);
			DynamicMaterialBeam->SetScalarParameterValue("XCells", XCells);
			DynamicMaterialBeam->SetScalarParameterValue("YCells", YCells);
			DynamicMaterialBeam->SetScalarParameterValue("CellWidth", MatrixWidth / XCells);
			DynamicMaterialBeam->SetScalarParameterValue("CellHeight", MatrixHeight / YCells);
			DynamicMaterialBeam->SetTextureParameterValue("MatrixData", MatrixDataTexture);
			MatrixHead->SetMaterial(1, DynamicMaterialBeam);
		}

		// Initialize components
		for (UDMXFixtureComponent* DMXComponent : TInlineComponentArray<UDMXFixtureComponent*>(this))
		{
			DMXComponent->Initialize();
		}

		SetDefaultMatrixFixtureState();
		UpdateDynamicTexture();

		HasBeenInitialized = true;
	}
	else
	{
		HasBeenInitialized = false;
	}
}

// DMX Data is packed based on this convention
// texture row index 0: RGBColor / Dimmer (4 channels total)
// texture row index 1: Pan / Tilt  (2 channels total)
void ADMXFixtureActorMatrix::UpdateMatrixData(int32 RowIndex, int32 CellIndex, int32 ChannelIndex, float Value)
{
	int32 Index = (RowIndex * XCells * YCells * 4) + (CellIndex * 4) + ChannelIndex;
	if (Index < MatrixDataSize)
	{
		MatrixData[Index] = FMath::RoundToInt(Value * 255);
	}
}

// NB: Matrix data and effects are hardcoded for now - We could expose that to BP later
void ADMXFixtureActorMatrix::PushFixtureMatrixCellData(TArray<FDMXCell> Cells)
{
	SCOPE_CYCLE_COUNTER(STAT_FixtureActorMatrixPushFixtureMatrixCellData);

	if (HasBeenInitialized)
	{
		// get fixture patch
		UDMXEntityFixturePatch* FixturePatch = DMX->GetFixturePatch();

		if (FixturePatch)
		{
			for (int32 CellIndex = 0; CellIndex < Cells.Num(); CellIndex++)
			{
				TMap<FDMXAttributeName, float> NormalizedValuePerAttribute;
				FixturePatch->GetNormalizedMatrixCellValues(Cells[CellIndex].Coordinate, NormalizedValuePerAttribute);

				for (UDMXFixtureComponent* DMXComponent : TInlineComponentArray<UDMXFixtureComponent*>(this))
				{
					if (DMXComponent->bIsEnabled && DMXComponent->bUsingMatrixData)
					{	
						// set current cell reference
						if (bIgnorePixelMappingDistributionOfFixturePatch)
						{
							DMXComponent->SetCurrentCell(Cells[CellIndex].CellID - 1);
						}
						else
						{
							DMXComponent->SetCurrentCell(CellIndex);
						}

						if (UDMXFixtureComponentColor* ColorComponent = Cast<UDMXFixtureComponentColor>(DMXComponent))
						{
							// Color component
							if (FLinearColor* CurrentTargetColorPtr = ColorComponent->CurrentTargetColorRef)
							{
								const float* FirstTargetValuePtr = NormalizedValuePerAttribute.Find(ColorComponent->DMXChannel1);
								const float* SecondTargetValuePtr = NormalizedValuePerAttribute.Find(ColorComponent->DMXChannel2);
								const float* ThirdTargetValuePtr = NormalizedValuePerAttribute.Find(ColorComponent->DMXChannel3);
								const float* FourthTargetValuePtr = NormalizedValuePerAttribute.Find(ColorComponent->DMXChannel4);

								// 1.f if channel not found
								const float r = (FirstTargetValuePtr) ? *FirstTargetValuePtr : CurrentTargetColorPtr->R;
								const float g = (SecondTargetValuePtr) ? *SecondTargetValuePtr : CurrentTargetColorPtr->G;
								const float b = (ThirdTargetValuePtr) ? *ThirdTargetValuePtr : CurrentTargetColorPtr->B;
								const float a = (FourthTargetValuePtr) ? *FourthTargetValuePtr : CurrentTargetColorPtr->A;

								FLinearColor NewTargetColor(r, g, b, a);
								if (ColorComponent->IsColorValid(NewTargetColor))
								{
									ColorComponent->SetTargetColor(NewTargetColor);

									// pack data in Matrix structure
									UpdateMatrixData(0, CellIndex, 0, NewTargetColor.B);
									UpdateMatrixData(0, CellIndex, 1, NewTargetColor.G);
									UpdateMatrixData(0, CellIndex, 2, NewTargetColor.R);
									UpdateMatrixData(0, CellIndex, 3, NewTargetColor.A);
								}
							}
						}
						else if (UDMXFixtureComponentSingle* SingleComponent = Cast<UDMXFixtureComponentSingle>(DMXComponent))
						{
							// Single channel component - hardcoded for now
							float* D1 = NormalizedValuePerAttribute.Find(SingleComponent->DMXChannel.Name.Name);
							if (D1)
							{
								if (SingleComponent->DMXChannel.Name.Name == FName("Dimmer"))
								{
									const float TargetValue = SingleComponent->NormalizedToAbsoluteValue(*D1);
									if (SingleComponent->IsTargetValid(TargetValue))
									{
										UpdateMatrixData(0, CellIndex, 3, TargetValue);
									}
								}
								else if (SingleComponent->DMXChannel.Name.Name == FName("Pan"))
								{
									const float TargetValue = SingleComponent->NormalizedToAbsoluteValue(*D1);
									if (SingleComponent->IsTargetValid(TargetValue))
									{
										UpdateMatrixData(1, CellIndex, 0, TargetValue);
									}
								}
								else if (SingleComponent->DMXChannel.Name.Name == FName("Tilt"))
								{
									const float TargetValue = SingleComponent->NormalizedToAbsoluteValue(*D1);
									if (SingleComponent->IsTargetValid(TargetValue))
									{
										UpdateMatrixData(1, CellIndex, 1, TargetValue);
									}
								}
							}
						}
					}
				}
			}
		}

		// push matrix data in dynamic texture
		UpdateDynamicTexture();

		// set light color
		FLinearColor MatrixAverageColor = GetMatrixAverageColor();
		SpotLight->SetLightColor(MatrixAverageColor, false);
		SpotLight->SetIntensity(LightIntensityMax * MatrixAverageColor.A);
	}
}

FLinearColor ADMXFixtureActorMatrix::GetMatrixAverageColor()
{
	FLinearColor AverageColor(0,0,0,0);
	int NbrCells = XCells * YCells;
	for (int i = 0; i < NbrCells; i++)
	{
		AverageColor.B += (float)MatrixData[i*4] / 255.0f;
		AverageColor.G += (float)MatrixData[(i*4) + 1] / 255.0f;
		AverageColor.R += (float)MatrixData[(i*4) + 2] / 255.0f;
		AverageColor.A += (float)MatrixData[(i*4) + 3] / 255.0f;
	}
	AverageColor = AverageColor / NbrCells;
	return AverageColor;
}

void ADMXFixtureActorMatrix::UpdateDynamicTexture()
{
	if (MatrixDataTexture)
	{
		int NbrCells = XCells * YCells;
		UpdateMatrixTexture(MatrixData.GetData(), MatrixDataTexture, 0, 1, TextureRegion, NbrCells * 4, 4);
	}
}

/*
void ADMXFixtureActorMatrix::PostLoad()
{
	Super::PostLoad();
	//GenerateMatrixMesh();
}
*/

void ADMXFixtureActorMatrix::GenerateMatrixMesh()
{
	MatrixHead->ClearAllMeshSections();
	GenerateMatrixCells();
	GenerateMatrixBeam();
	MatrixHead->SetRelativeLocation(FVector(MatrixWidth*-0.5f, MatrixHeight * -0.5f, MatrixDepth * 0.5f));
}

void ADMXFixtureActorMatrix::SetDefaultMatrixFixtureState()
{
	if (UDMXEntityFixturePatch* FixturePatch = DMX->GetFixturePatch())
	{
		TArray<FDMXCell> Cells;
		FixturePatch->GetAllMatrixCells(Cells);

		TArray<UDMXFixtureComponent*> DMXComponents;
		GetComponents(DMXComponents);

		for (int CurrentCellIndex = 0; CurrentCellIndex < Cells.Num(); CurrentCellIndex++)
		{
			const FDMXCell& Cell = Cells[CurrentCellIndex];

			for (UDMXFixtureComponent* DMXComponent : DMXComponents)
			{
				if (DMXComponent->bIsEnabled && DMXComponent->bUsingMatrixData)
				{
					// set current cell reference
					DMXComponent->SetCurrentCell(CurrentCellIndex);

					if (UDMXFixtureComponentColor* ColorComponent = Cast<UDMXFixtureComponentColor>(DMXComponent))
					{						
						// Color component
						FLinearColor DefaultColor = FLinearColor::Black;
						if (ColorComponent->IsColorValid(DefaultColor))
						{
							ColorComponent->SetTargetColor(DefaultColor);

							// pack data in Matrix structure
							UpdateMatrixData(0, CurrentCellIndex, 0, DefaultColor.B);
							UpdateMatrixData(0, CurrentCellIndex, 1, DefaultColor.G);
							UpdateMatrixData(0, CurrentCellIndex, 2, DefaultColor.R);
							UpdateMatrixData(0, CurrentCellIndex, 3, DefaultColor.A);
						}
					}
					else if (UDMXFixtureComponentSingle* SingleComponent = Cast<UDMXFixtureComponentSingle>(DMXComponent))
					{
						// Single Component
						float TargetValue = SingleComponent->DMXChannel.DefaultValue;
						SingleComponent->SetTargetValue(TargetValue);
						SingleComponent->SetValueNoInterp(TargetValue);
					}
					else if (UDMXFixtureComponentDouble* DoubleComponent = Cast<UDMXFixtureComponentDouble>(DMXComponent))
					{
						// Double Component
						float Channel1TargetValue = DoubleComponent->DMXChannel1.DefaultValue;
						float Channel2TargetValue = DoubleComponent->DMXChannel2.DefaultValue;
						DoubleComponent->SetTargetValue(0, Channel1TargetValue);
						DoubleComponent->SetChannel1ValueNoInterp(Channel1TargetValue);
						DoubleComponent->SetTargetValue(1, Channel2TargetValue);
						DoubleComponent->SetChannel2ValueNoInterp(Channel2TargetValue);
					}
				}
			}
		}

		// push matrix data in dynamic texture
		UpdateDynamicTexture();

		// set light color
		FLinearColor MatrixAverageColor = GetMatrixAverageColor();
		SpotLight->SetLightColor(MatrixAverageColor, false);
		SpotLight->SetIntensity(LightIntensityMax * MatrixAverageColor.A);
	}
}

void ADMXFixtureActorMatrix::GenerateEditorMatrixMesh()
{
	if (DMX && !GWorld->HasBegunPlay())
	{
		if (UDMXEntityFixturePatch* FixturePatch = DMX->GetFixturePatch())
		{
			FDMXFixtureMatrix MatrixProperties;
			FixturePatch->GetMatrixProperties(MatrixProperties);
			XCells = MatrixProperties.XCells;
			YCells = MatrixProperties.YCells;

			// Limit cells [1-DMX_UNIVERSE_SIZE]
			constexpr int DMXUniverseSize = DMX_UNIVERSE_SIZE;

			XCells = FMath::Max(XCells, 1);
			YCells = FMath::Max(YCells, 1);
			XCells = FMath::Min(XCells, DMXUniverseSize);
			YCells = FMath::Min(YCells, DMXUniverseSize);

			MatrixHead->ClearAllMeshSections();
			GenerateMatrixCells();
			GenerateMatrixBeam();
			MatrixHead->SetRelativeLocation(FVector(MatrixWidth * -0.5f, MatrixHeight * -0.5f, MatrixDepth * 0.5f));

			// Assign MIC
			MatrixHead->SetMaterial(0, LensMaterialInstance);
			MatrixHead->SetMaterial(1, BeamMaterialInstance);
		}
	}
}

void ADMXFixtureActorMatrix::GenerateMatrixCells()
{
	// Reset arrays
	Vertices.Reset();
	Triangles.Reset();
	Normals.Reset();
	Tangents.Reset();
	UV0.Reset();
	UV1.Reset();
	UV2.Reset();
	Colors.Reset();
	QuadIndexCount = 0;

	// Quad 3d Positions
	FVector TopLeftPosition(0, 1, 0);
	FVector BottomLeftPosition(0, 0, 0);
	FVector BottomRightPosition(1, 0, 0);
	FVector TopRightPosition(1, 1, 0);

	// Quad 2d UVs
	FVector2D TopLeftUV(0, 1);
	FVector2D BottomLeftUV(0, 0);
	FVector2D BottomRightUV(1, 0);
	FVector2D TopRightUV(1, 1);

	FProcMeshTangent Tangent = FProcMeshTangent(1.0f, 0.0f, 0.0f);

	// Normal
	FVector Normal = FVector::CrossProduct(TopLeftPosition-BottomRightPosition, TopLeftPosition-TopRightPosition).GetSafeNormal();

	// Quads following [topLeft -> bottomRight] convention
	float QuadWidth = MatrixWidth / XCells;
	float QuadHeight = MatrixHeight / YCells;
	float RowOffset = 0;
	float ColumnOffset = 0;
	for (int RowIndex = 0; RowIndex < YCells; RowIndex++)
	{
		RowOffset = RowIndex * QuadHeight;
		for (int ColumnIndex = 0; ColumnIndex < XCells; ColumnIndex++)
		{
			ColumnOffset = ColumnIndex * QuadWidth;

			FVector P1 = TopLeftPosition;
			P1.X = (P1.X*QuadWidth) + ColumnOffset;
			P1.Y = (P1.Y*QuadHeight) + RowOffset;

			FVector P2 = BottomLeftPosition;
			P2.X = (P2.X * QuadWidth) + ColumnOffset;
			P2.Y = (P2.Y * QuadHeight) + RowOffset;

			FVector P3 = BottomRightPosition;
			P3.X = (P3.X * QuadWidth) + ColumnOffset;
			P3.Y = (P3.Y * QuadHeight) + RowOffset;

			FVector P4= TopRightPosition;
			P4.X = (P4.X * QuadWidth) + ColumnOffset;
			P4.Y = (P4.Y * QuadHeight) + RowOffset;

			Vertices.Add(P1);
			Vertices.Add(P2);
			Vertices.Add(P3);
			Vertices.Add(P4);

			int IndexOffset = QuadIndexCount * 4;
			Triangles.Add(0 + IndexOffset);
			Triangles.Add(2 + IndexOffset);
			Triangles.Add(1 + IndexOffset);
			Triangles.Add(0 + IndexOffset);
			Triangles.Add(3 + IndexOffset);
			Triangles.Add(2 + IndexOffset);

			for (int i = 0; i < 4; i++)
			{
				Normals.Add(Normal);
				Tangents.Add(Tangent);
				Colors.Add(FColor(255, 255, 255));
			}

			// UVs to sample lens mask
			UV0.Add(TopLeftUV);
			UV0.Add(BottomLeftUV);
			UV0.Add(BottomRightUV);
			UV0.Add(TopRightUV);

			// UVs to sample first row, targetting middle of pixel
			//float UOffset = QuadIndexCount / float(NbrCells);
			//float UHalfOffset = (1.0f/NbrCells) * 0.5f;
			//UV1.Add(FVector2D(UOffset + UHalfOffset, 0.5));
			//UV1.Add(FVector2D(UOffset + UHalfOffset, 0.5));
			//UV1.Add(FVector2D(UOffset + UHalfOffset, 0.5));
			//UV1.Add(FVector2D(UOffset + UHalfOffset, 0.5));

			// Pack QuadIndexCount value into two 8bits
			// decoding: HighByte*256 + LowByte
			uint8 HighByte = FMath::Floor(QuadIndexCount / 256.0f);
			uint8 LowByte = QuadIndexCount % 256;
			UV1.Add(FVector2D(HighByte, LowByte));
			UV1.Add(FVector2D(HighByte, LowByte));
			UV1.Add(FVector2D(HighByte, LowByte));
			UV1.Add(FVector2D(HighByte, LowByte));

			// UVs to specify if vertex is part of "lens=1" or "chassis=0"
			UV2.Add(FVector2D(1, 1));
			UV2.Add(FVector2D(1, 1));
			UV2.Add(FVector2D(1, 1));
			UV2.Add(FVector2D(1, 1));

			QuadIndexCount++;
		}
	}

	// Create Matrix Chassis
	FVector MatrixTopLeftPosition(0, 0, 0);
	FVector MatrixBottomLeftPosition(0, QuadHeight*YCells, 0);
	FVector MatrixBottomRightPosition(MatrixWidth, QuadHeight*YCells, 0);
	FVector MatrixTopRightPosition(MatrixWidth, 0, 0);
	GenerateMatrixChassis(MatrixTopLeftPosition, MatrixBottomLeftPosition, MatrixBottomRightPosition, MatrixTopRightPosition);

	// Create mesh section 0
	MatrixHead->CreateMeshSection(0, Vertices, Triangles, Normals, UV0, UV1, UV2, UV0, Colors, Tangents, false);
}


void ADMXFixtureActorMatrix::GenerateMatrixBeam()
{
	// Reset arrays
	Vertices.Reset();
	Triangles.Reset();
	Normals.Reset();
	Tangents.Reset();
	UV0.Reset();
	UV1.Reset();
	UV2.Reset();
	Colors.Reset();
	
	float Quality = 1.0f;
	switch (QualityLevel)
	{
		case(EDMXFixtureQualityLevel::LowQuality): Quality = 0.25f; break;
		case(EDMXFixtureQualityLevel::MediumQuality): Quality = 0.5f; break;
		case(EDMXFixtureQualityLevel::HighQuality): Quality = 1.0f; break;
		case(EDMXFixtureQualityLevel::UltraQuality): Quality = 2.0f; break;
		default: Quality = 1.0f;
	}

	int NbrSamples = FMath::CeilToInt(Quality * 4);

	// Quad 3d directions from center position
	FVector TopLeftDirection(-1, 1, 0);
	FVector BottomLeftDirection(-1,-1, 0);
	FVector BottomRightDirection(1, -1, 0);
	FVector TopRightDirection(1, 1, 0);

	// Quad 2d UVs
	FVector2D TopLeftUV(0, 1);
	FVector2D BottomLeftUV(0, 0);
	FVector2D BottomRightUV(1, 0);
	FVector2D TopRightUV(1, 1);

	// Tangent and normal
	FProcMeshTangent Tangent = FProcMeshTangent(1.0f, 0.0f, 0.0f);
	FVector Normal = FVector(0, 0, 1);

	// Build quads
	float MaxDistance = FMath::Sqrt(MatrixWidth * MatrixHeight) * 0.33f;
	MaxDistance = FMath::Min(MaxDistance, 50.0f);

	float QuadDistance = MaxDistance / NbrSamples;
	float QuadWidth = MatrixWidth / XCells;
	float QuadHeight = MatrixHeight / YCells;
	float StartX = QuadWidth * 0.5f;
	float StartY = QuadHeight * 0.5f;
	float RowOffset = 0;
	float ColumnOffset = 0;
	float QuadScaleIncrement = 1.5f / NbrSamples;
	float QuadScale = 1.0f;
	FVector QuadSize(QuadWidth*0.5f, QuadHeight*0.5f, 0);

	int QuadCount = 0;
	for (int SampleIndex = 0; SampleIndex < NbrSamples; SampleIndex++)
	{
		int QuadIndex = 0;
		QuadScale += QuadScaleIncrement;
		for (int RowIndex = 0; RowIndex < YCells; RowIndex++)
		{
			RowOffset = RowIndex * QuadHeight;
			for (int ColumnIndex = 0; ColumnIndex < XCells; ColumnIndex++)
			{
				ColumnOffset = ColumnIndex * QuadWidth;

				// Pack QuadIndex value into two 8bits
				// decoding: HighByte*256 + LowByte
				uint8 HighByte = FMath::Floor(QuadIndex / 256.0f);
				uint8 LowByte = QuadIndex % 256;

				// Positions
				FVector CenterPosition(StartX + ColumnOffset, StartY + RowOffset, 1.0f + (QuadDistance * SampleIndex));
				FVector P1 = CenterPosition + (TopLeftDirection * QuadSize * QuadScale);
				FVector P2 = CenterPosition + (BottomLeftDirection * QuadSize * QuadScale);
				FVector P3 = CenterPosition + (BottomRightDirection * QuadSize * QuadScale);
				FVector P4 = CenterPosition + (TopRightDirection * QuadSize * QuadScale);

				Vertices.Add(P1);
				Vertices.Add(P2);
				Vertices.Add(P3);
				Vertices.Add(P4);

				// Triangles
				int IndexOffset = QuadCount * 4;
				Triangles.Add(0 + IndexOffset);
				Triangles.Add(2 + IndexOffset);
				Triangles.Add(1 + IndexOffset);
				Triangles.Add(0 + IndexOffset);
				Triangles.Add(3 + IndexOffset);
				Triangles.Add(2 + IndexOffset);

				for (int i = 0; i < 4; i++)
				{
					Normals.Add(Normal);
					Tangents.Add(Tangent);
					Colors.Add(FColor(255, 255, 255));
					UV1.Add(FVector2D(HighByte, LowByte));
					UV2.Add(FVector2D(1, 1));
				}

				// UVs to sample lens mask
				UV0.Add(TopLeftUV);
				UV0.Add(BottomLeftUV);
				UV0.Add(BottomRightUV);
				UV0.Add(TopRightUV);

				QuadIndex++;
				QuadCount++;
			}
		}
	}

	// Create mesh section 1
	MatrixHead->CreateMeshSection(1, Vertices, Triangles, Normals, UV0, UV1, UV2, UV0, Colors, Tangents, false);
}

void ADMXFixtureActorMatrix::GenerateMatrixChassis(FVector TL, FVector BL, FVector BR, FVector TR)
{
	// Create 5 faces to close matrix box
	FVector Depth(0, 0, MatrixDepth);
	FProcMeshTangent Tangent = FProcMeshTangent(1.0f, 0.0f, 0.0f);
	
	// bottom face
	AddQuad(TL-Depth, BL-Depth, BR-Depth, TR-Depth, Tangent);

	// side 1
	FVector P1 = BL;
	FVector P2 = BL - Depth;
	FVector P3 = BR - Depth;
	FVector P4 = BR;
	AddQuad(P1, P4, P3, P2, Tangent);

	// side 2
	P1 = TL;
	P2 = TL - Depth;
	P3 = BL - Depth;
	P4 = BL;
	AddQuad(P1, P4, P3, P2, Tangent);

	// side 3
	P1 = TR;
	P2 = TR - Depth;
	P3 = TL - Depth;
	P4 = TL;
	AddQuad(P1, P4, P3, P2, Tangent);

	// side 4
	P1 = BR;
	P2 = BR - Depth;
	P3 = TR - Depth;
	P4 = TR;
	AddQuad(P1, P4, P3, P2, Tangent);
}

void ADMXFixtureActorMatrix::AddQuad(FVector TL, FVector BL, FVector BR, FVector TR, FProcMeshTangent Tangent)
{
	Vertices.Add(TL);
	Vertices.Add(BL);
	Vertices.Add(BR);
	Vertices.Add(TR);

	int IndexOffset = QuadIndexCount * 4;
	Triangles.Add(0 + IndexOffset);
	Triangles.Add(2 + IndexOffset);
	Triangles.Add(1 + IndexOffset);
	Triangles.Add(0 + IndexOffset);
	Triangles.Add(3 + IndexOffset);
	Triangles.Add(2 + IndexOffset);

	FVector Normal = FVector::CrossProduct(TL-BR, TL-TR).GetSafeNormal();
	for (int i = 0; i < 4; i++)
	{
		Normals.Add(Normal);
		Tangents.Add(Tangent);
		Colors.Add(FColor(255, 255, 255));
		UV0.Add(FVector2D(0, 0));
		UV1.Add(FVector2D(0, 0));
		UV2.Add(FVector2D(0, 0)); // "chassis=0"
	}

	QuadIndexCount++;
}
