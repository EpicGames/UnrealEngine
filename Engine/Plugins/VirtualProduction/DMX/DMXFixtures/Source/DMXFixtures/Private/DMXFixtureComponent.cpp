// Copyright Epic Games, Inc. All Rights Reserved.


#include "DMXFixtureComponent.h"

#include "DMXFixtureActorMatrix.h"

#include "DMXInterpolation.h"
#include "RenderingThread.h"
#include "Engine/Texture2D.h"
#include "Rendering/Texture2DResource.h"


UDMXFixtureComponent::UDMXFixtureComponent()
{
	PrimaryComponentTick.bCanEverTick = false;
}

void UDMXFixtureComponent::InitCells(int NumCells)
{
	Cells.Init(FCell(), NumCells);
	for (FCell& Cell : Cells)
	{
		Cell.ChannelInterpolation.Init(FInterpolationData(), 1);
	}
}

void UDMXFixtureComponent::Initialize()
{
	ADMXFixtureActorMatrix* ParentMatrixFixture = Cast<ADMXFixtureActorMatrix>(GetOwner());
	if (ParentMatrixFixture)
	{
		bUsingMatrixData = true;

		// Init cells for DMX Matrix Fixtures
		int nCells = ParentMatrixFixture->XCells * ParentMatrixFixture->YCells;
		InitCells(nCells);

		if (nCells > 0)
		{
			CurrentCell = &Cells[0];
		}
	}
	else
	{		
		// Init cells for other parent actors (only one cell)
		InitCells(1);
		CurrentCell = &Cells[0];
	}
}

void UDMXFixtureComponent::SetCurrentCell(int Index)
{
	if (Cells.IsValidIndex(Index))
	{
		CurrentCell = &Cells[Index];
	}
}

void UDMXFixtureComponent::ApplySpeedScale()
{
	for (auto& Cell : Cells)
	{
		for (auto& ChannelInterp : Cell.ChannelInterpolation)
		{
			ChannelInterp.InterpolationScale = InterpolationScale;
		}
	}
}

ADMXFixtureActor* UDMXFixtureComponent::GetParentFixtureActor()
{
	AActor* Parent = GetOwner();
	if (Parent)
	{ 
		ADMXFixtureActor* ParentFixtureActor = Cast<ADMXFixtureActor>(Parent);
		return ParentFixtureActor;
	}
	else
	{
		return nullptr;
	}
}

// Warning: Input Texture must use VectorDisplacementMap compression or else RHI complains about BytesPerPixel
// Reads pixel color in the middle of each "Texture" and output linear colors
TArray<FLinearColor> UDMXFixtureComponent::GetTextureCenterColors(UTexture2D* Texture, int NumCells, bool bUpdateTextureResource)
{
	TArray<FLinearColor> PixelColorArray;
	PixelColorArray.SetNumZeroed(NumCells, EAllowShrinking::Yes);
	if (Texture)
	{
		if (bUpdateTextureResource)
		{
			Texture->UpdateResource();
		}

		FRenderCommandFence Fence;
		ENQUEUE_RENDER_COMMAND(GetPixelColors)
		(
			[Texture, &PixelColorArray, NumCells](FRHICommandListImmediate& RHICmdList)
			{
				FTexture2DResource* uTex2DRes = (FTexture2DResource*)Texture->GetResource();
				if (uTex2DRes)
				{
					FIntVector Size = uTex2DRes->TextureRHI->GetSizeXYZ();
					TArray<FColor> Data;
					FIntRect Rect(0, 0, Size.X, Size.Y);
					RHICmdList.ReadSurfaceData(uTex2DRes->TextureRHI, Rect, Data, FReadSurfaceDataFlags());

					// sample pixel in the middle of each 'cell'
					TArray<FLinearColor> Out;
					int32 V = FMath::FloorToInt(Size.Y * 0.5f) - 1;
					for (int32 i = 0; i < NumCells; i++)
					{
						int32 U = (((i / float(NumCells)) + (0.5f / NumCells)) * Size.X) - 1;
						Out.Add(Data[V * Size.X + U]);
					}

					PixelColorArray = std::move(Out);
				}
			}
		);

		// wait for render thread to finish
		Fence.BeginFence();
		Fence.Wait();
	}


	// color correction
	for (auto& Pixel : PixelColorArray)
	{
		Pixel.R = FMath::Pow(Pixel.R, 0.4545f);
		Pixel.G = FMath::Pow(Pixel.G, 0.4545f);
		Pixel.B = FMath::Pow(Pixel.B, 0.4545f);
	}


	return PixelColorArray;
}
