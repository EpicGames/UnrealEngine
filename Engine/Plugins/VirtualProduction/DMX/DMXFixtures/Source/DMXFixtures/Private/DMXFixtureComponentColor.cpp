// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixtureComponentColor.h"

#include "DMXTypes.h"


UDMXFixtureComponentColor::UDMXFixtureComponentColor()
	: CurrentTargetColorRef(nullptr)
{
	PrimaryComponentTick.bCanEverTick = false;
}

void UDMXFixtureComponentColor::PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttribute)
{
	if (FLinearColor* CurrentTargetColorPtr = CurrentTargetColorRef)
	{
		const float* FirstTargetValuePtr = ValuePerAttribute.Map.Find(DMXChannel1);
		const float* SecondTargetValuePtr = ValuePerAttribute.Map.Find(DMXChannel2);
		const float* ThirdTargetValuePtr = ValuePerAttribute.Map.Find(DMXChannel3);
		const float* FourthTargetValuePtr = ValuePerAttribute.Map.Find(DMXChannel4);

		// Current color if channel not found
		const float r = (FirstTargetValuePtr) ? *FirstTargetValuePtr : CurrentTargetColorPtr->R;
		const float g = (SecondTargetValuePtr) ? *SecondTargetValuePtr : CurrentTargetColorPtr->G;
		const float b = (ThirdTargetValuePtr) ? *ThirdTargetValuePtr : CurrentTargetColorPtr->B;
		const float a = (FourthTargetValuePtr) ? *FourthTargetValuePtr : CurrentTargetColorPtr->A;

		FLinearColor NewTargetColor(r, g, b, a);
		if (IsColorValid(NewTargetColor))
		{
			SetTargetColor(NewTargetColor);
		}
	}
}

void UDMXFixtureComponentColor::GetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames)
{
	if (bIsEnabled)
	{
		OutAttributeNames.Add(DMXChannel1.Name);
		OutAttributeNames.Add(DMXChannel2.Name);
		OutAttributeNames.Add(DMXChannel3.Name);
		OutAttributeNames.Add(DMXChannel4.Name);
	}
}

void UDMXFixtureComponentColor::Initialize()
{
	Super::Initialize();

	const FLinearColor& DefaultColor = FLinearColor::White;
	TargetColorArray.Init(DefaultColor, Cells.Num());
	
	CurrentTargetColorRef = &TargetColorArray[0];

	InitializeComponent();

	SetColorNoInterp(DefaultColor);
}

void UDMXFixtureComponentColor::SetCurrentCell(int Index)
{
	if (CurrentTargetColorRef &&
		TargetColorArray.IsValidIndex(Index) &&
		Index < TargetColorArray.Num())
	{
		CurrentTargetColorRef = &TargetColorArray[Index];
	}
}

bool UDMXFixtureComponentColor::IsColorValid(const FLinearColor& NewColor) const
{
	if (CurrentTargetColorRef &&
		!CurrentTargetColorRef->Equals(NewColor, SkipThreshold))
	{
		return true;
	}

	return false;
}

void UDMXFixtureComponentColor::SetTargetColor(const FLinearColor& NewColor)
{
	if (CurrentTargetColorRef &&
		IsColorValid(NewColor))
	{
		// Never interpolated
		CurrentTargetColorRef->R = NewColor.R;
		CurrentTargetColorRef->G = NewColor.G;
		CurrentTargetColorRef->B = NewColor.B;
		CurrentTargetColorRef->A = NewColor.A;

		SetColorNoInterp(NewColor);
	}
}
