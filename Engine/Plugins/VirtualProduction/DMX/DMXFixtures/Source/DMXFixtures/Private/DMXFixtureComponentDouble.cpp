// Copyright Epic Games, Inc. All Rights Reserved.


#include "DMXFixtureComponentDouble.h"

#include "DMXInterpolation.h"
#include "DMXTypes.h"


UDMXFixtureComponentDouble::UDMXFixtureComponentDouble()
{
	PrimaryComponentTick.bCanEverTick = false;
}

void UDMXFixtureComponentDouble::PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttribute)
{
	const float* FirstTargetValuePtr = ValuePerAttribute.Map.Find(DMXChannel1.Name);
	if (FirstTargetValuePtr)
	{
		const float Channel1RemappedValue = NormalizedToAbsoluteValue(0, *FirstTargetValuePtr);

		constexpr int32 FirstChannelIndex = 0;
		SetTargetValue(FirstChannelIndex, Channel1RemappedValue);
	}

	const float* SecondTargetValuePtr = ValuePerAttribute.Map.Find(DMXChannel2.Name);
	if (SecondTargetValuePtr)
	{
		const float Channel2RemappedValue = NormalizedToAbsoluteValue(1, *SecondTargetValuePtr);

		constexpr int32 SecondChannelIndex = 1;
		SetTargetValue(SecondChannelIndex, Channel2RemappedValue);
	}
}

void UDMXFixtureComponentDouble::GetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames)
{
	if (bIsEnabled)
	{
		OutAttributeNames.Add(DMXChannel1.Name.Name);
		OutAttributeNames.Add(DMXChannel2.Name.Name);
	}
}

void UDMXFixtureComponentDouble::Initialize()
{
	Super::Initialize();

	// Two channels per cell
	for (auto& Cell : Cells)
	{
		// Init Channel interpolation  
		Cell.ChannelInterpolation.Init(FInterpolationData(), 2);

		// Init interpolation speed scale
		Cell.ChannelInterpolation[0].InterpolationScale = InterpolationScale;
		Cell.ChannelInterpolation[1].InterpolationScale = InterpolationScale;

		// Init interpolation range 
		Cell.ChannelInterpolation[0].RangeValue = FMath::Abs(DMXChannel1.MaxValue - DMXChannel1.MinValue);
		Cell.ChannelInterpolation[1].RangeValue = FMath::Abs(DMXChannel2.MaxValue - DMXChannel2.MinValue);

		// Set the default value as the target and the current
		Cell.ChannelInterpolation[0].CurrentValue = DMXChannel1.DefaultValue;
		Cell.ChannelInterpolation[1].CurrentValue = DMXChannel2.DefaultValue;

		Cell.ChannelInterpolation[0].TargetValue = DMXChannel1.DefaultValue;
		Cell.ChannelInterpolation[1].TargetValue = DMXChannel2.DefaultValue;
	}

	InitializeComponent();

	SetChannel1ValueNoInterp(DMXChannel1.DefaultValue);
	SetChannel2ValueNoInterp(DMXChannel2.DefaultValue);
}

float UDMXFixtureComponentDouble::GetDMXInterpolatedStep(int32 ChannelIndex) const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 2)
	{
		const FInterpolationData& InterpolationData = ChannelIndex == 0 ? CurrentCell->ChannelInterpolation[0] : CurrentCell->ChannelInterpolation[1];
		return InterpolationData.CurrentSpeed * InterpolationData.Direction;
	}

	return 0.f;
}

float UDMXFixtureComponentDouble::GetDMXInterpolatedValue(int32 ChannelIndex) const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 2)
	{
		const FInterpolationData& InterpolationData = ChannelIndex == 0 ? CurrentCell->ChannelInterpolation[0] : CurrentCell->ChannelInterpolation[1];
		return InterpolationData.CurrentValue;
	}

	return 0.f;
}

float UDMXFixtureComponentDouble::GetDMXTargetValue(int32 ChannelIndex) const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 2)
	{
		const FInterpolationData& InterpolationData = ChannelIndex == 0 ? CurrentCell->ChannelInterpolation[0] : CurrentCell->ChannelInterpolation[1];

		return InterpolationData.TargetValue;
	}

	return 0.f;
}

bool UDMXFixtureComponentDouble::IsDMXInterpolationDone(int32 ChannelIndex) const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 2)
	{
		const FInterpolationData& InterpolationData = ChannelIndex == 0 ? CurrentCell->ChannelInterpolation[0] : CurrentCell->ChannelInterpolation[1];

		return InterpolationData.IsInterpolationDone();
	}

	return 0.f;
}

float UDMXFixtureComponentDouble::NormalizedToAbsoluteValue(int32 ChannelIndex, float Alpha) const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 2)
	{
		const FDMXChannelData& ChannelData = ChannelIndex == 0 ? DMXChannel1 : DMXChannel2;

		const float AbsoluteValue = FMath::Lerp(ChannelData.MinValue, ChannelData.MaxValue, Alpha);

		return AbsoluteValue;
	}

	return 0.f;
}

bool UDMXFixtureComponentDouble::IsTargetValid(int32 ChannelIndex, float Target)
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 2)
	{
		const FInterpolationData& InterpolationData = ChannelIndex == 0 ? CurrentCell->ChannelInterpolation[0] : CurrentCell->ChannelInterpolation[1];

		return InterpolationData.IsTargetValid(Target, SkipThreshold);
	}

	return false;
}

void UDMXFixtureComponentDouble::SetTargetValue(int32 ChannelIndex, float AbsoluteValue)
{
	if (CurrentCell && 
		CurrentCell->ChannelInterpolation.Num() == 2 &&
		IsTargetValid(ChannelIndex, AbsoluteValue))
	{
		FInterpolationData& InterpolationData = ChannelIndex == 0 ? CurrentCell->ChannelInterpolation[0] : CurrentCell->ChannelInterpolation[1];

		if (bUseInterpolation)
		{
			if (InterpolationData.bFirstValueWasSet)
			{				
				// Only 'Push' the next value into interpolation. BPs will read the resulting value on tick.
				InterpolationData.Push(AbsoluteValue);
			}
			else
			{
				// Jump to the first value if it never was set
				InterpolationData.SetValueNoInterp(AbsoluteValue);
				InterpolationData.bFirstValueWasSet = true;

				if (ChannelIndex == 0)
				{
					SetChannel1ValueNoInterp(AbsoluteValue);
				}
				else
				{
					SetChannel2ValueNoInterp(AbsoluteValue);
				}
			}
		}
		else
		{
			InterpolationData.SetValueNoInterp(AbsoluteValue);

			if (ChannelIndex == 0)
			{
				SetChannel1ValueNoInterp(AbsoluteValue);
			}
			else
			{
				SetChannel2ValueNoInterp(AbsoluteValue);
			}
		}
	}
}	
