// Copyright Epic Games, Inc. All Rights Reserved.


#include "DMXFixtureComponentSingle.h"

#include "DMXInterpolation.h"
#include "DMXTypes.h"


UDMXFixtureComponentSingle::UDMXFixtureComponentSingle()
{
	PrimaryComponentTick.bCanEverTick = false;
}

void UDMXFixtureComponentSingle::PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttribute)
{
	const float* TargetValuePtr = ValuePerAttribute.Map.Find(DMXChannel.Name);
	if (TargetValuePtr)
	{
		const float RemappedValue = NormalizedToAbsoluteValue(*TargetValuePtr);
		SetTargetValue(RemappedValue);
	}
}

void UDMXFixtureComponentSingle::GetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames)
{
	if (bIsEnabled)
	{
		OutAttributeNames.Add(DMXChannel.Name.Name);
	}
}

void UDMXFixtureComponentSingle::Initialize()
{
	Super::Initialize();

	// 1 channel per cell
	for (auto& Cell : Cells)
	{
		// Init Channel interpolation  
		Cell.ChannelInterpolation.Init(FInterpolationData(), 1);

		// Init interpolation speed scale
		Cell.ChannelInterpolation[0].InterpolationScale = InterpolationScale;

		// Init interpolation range 
		Cell.ChannelInterpolation[0].RangeValue = FMath::Abs(DMXChannel.MaxValue - DMXChannel.MinValue);

		// Set the default value as the target and the current
		Cell.ChannelInterpolation[0].CurrentValue = DMXChannel.DefaultValue;
		Cell.ChannelInterpolation[0].TargetValue = DMXChannel.DefaultValue;

		Cell.ChannelInterpolation[0].StartTravel(DMXChannel.DefaultValue);
	}

	InitializeComponent();

	SetValueNoInterp(DMXChannel.DefaultValue);
}

float UDMXFixtureComponentSingle::GetDMXInterpolatedStep() const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 1)
	{
		return (CurrentCell->ChannelInterpolation[0].CurrentSpeed * CurrentCell->ChannelInterpolation[0].Direction);
	}

	return 0.f;
}

float UDMXFixtureComponentSingle::GetDMXInterpolatedValue() const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 1)
	{
		return CurrentCell->ChannelInterpolation[0].CurrentValue;
	}

	return 0.f;
}

float UDMXFixtureComponentSingle::GetDMXTargetValue() const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 1)
	{
		return CurrentCell->ChannelInterpolation[0].TargetValue;
	}

	return 0.f;
}

bool UDMXFixtureComponentSingle::IsDMXInterpolationDone() const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 1)
	{
		return CurrentCell->ChannelInterpolation[0].IsInterpolationDone();
	}

	return true;
}

float UDMXFixtureComponentSingle::NormalizedToAbsoluteValue(float Alpha) const
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 1)
	{
		const float AbsoluteValue = FMath::Lerp(DMXChannel.MinValue, DMXChannel.MaxValue, Alpha);

		return AbsoluteValue;
	}

	return 0.f;
}

bool UDMXFixtureComponentSingle::IsTargetValid(float Target)
{
	if (CurrentCell && CurrentCell->ChannelInterpolation.Num() == 1)
	{
		return CurrentCell->ChannelInterpolation[0].IsTargetValid(Target, SkipThreshold);
	}

	return false;
}

void UDMXFixtureComponentSingle::SetTargetValue(float AbsoluteValue)
{
	if (CurrentCell && 
		CurrentCell->ChannelInterpolation.Num() == 1 &&
		IsTargetValid(AbsoluteValue))
	{
		if (bUseInterpolation)
		{
			if (CurrentCell->ChannelInterpolation[0].bFirstValueWasSet)
			{
				// Only 'Push' the next value into interpolation. BPs will read the resulting value on tick.
				CurrentCell->ChannelInterpolation[0].Push(AbsoluteValue);
			}
			else
			{
				// Jump to the first value if it never was set
				CurrentCell->ChannelInterpolation[0].SetValueNoInterp(AbsoluteValue);
				CurrentCell->ChannelInterpolation[0].bFirstValueWasSet = true;

				SetValueNoInterp(AbsoluteValue);
			}
		}
		else
		{
			CurrentCell->ChannelInterpolation[0].SetValueNoInterp(AbsoluteValue);

			// Raise BP Event
			SetValueNoInterp(AbsoluteValue);
		}
	}
}
