// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXFixturesModule.h"

#define LOCTEXT_NAMESPACE "DMXFixturesModule"

void FDMXFixturesModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
}

void FDMXFixturesModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

IMPLEMENT_MODULE(FDMXFixturesModule, DMXFixtures)


#undef LOCTEXT_NAMESPACE
