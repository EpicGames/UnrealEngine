// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXFixtureActorBase.h"

#include "DMXFixtureComponent.h"
#include "DMXFixtureComponentDouble.h"
#include "DMXFixtureComponentSingle.h"
#include "DMXFixtureComponentColor.h"
#include "DMXMVRFixtureActorInterface.h"
#include "DMXProtocolTypes.h"
#include "Library/DMXEntityFixturePatch.h"

#include "CoreMinimal.h"
#include "Components/ArrowComponent.h"
#include "Components/SpotLightComponent.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "DMXFixtureActor.generated.h"


UENUM()
enum EDMXFixtureQualityLevel : int
{
	LowQuality			UMETA(DisplayName = "Low"),
	MediumQuality		UMETA(DisplayName = "Medium"),
	HighQuality			UMETA(DisplayName = "High"),
	UltraQuality		UMETA(DisplayName = "Ultra"),
	Custom				UMETA(DisplayName = "Custom")
};

UCLASS()
class DMXFIXTURES_API ADMXFixtureActor 
	: public ADMXFixtureActorBase
	, public IDMXMVRFixtureActorInterface
{
	GENERATED_BODY()

protected:
	//~ Begin AActor Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End AActor interface

public:
	ADMXFixtureActor();

	//~ Begin DMXMVRFixtureActorInterface interface
	virtual void OnMVRGetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames, TArray<FName>& OutMatrixAttributeNames) const override;
	//~ End DMXMVRFixtureActorInterface interface

	bool HasBeenInitialized;
	float LensRadius;
	void FeedFixtureData();

	// VISUAL QUALITY LEVEL----------------------

	// Visual quality level that changes the number of samples in the volumetric beam
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX Light Fixture", meta = (DisplayPriority = 0))
	TEnumAsByte<EDMXFixtureQualityLevel> QualityLevel;

	// Visual quality for the light beam - small value is visually better but cost more on GPU
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX Light Fixture", meta = (EditCondition = "QualityLevel == EDMXFixtureQualityLevel::Custom", EditConditionHides))
	float BeamQuality;

	// Visual quality for the light beam when zoom is wide - small value is visually better but cost more on GPU
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX Light Fixture", meta = (EditCondition = "QualityLevel == EDMXFixtureQualityLevel::Custom", EditConditionHides))
	float ZoomQuality;

	// HIERARCHY---------------------------------
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<USceneComponent> Base;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<USceneComponent> Yoke;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<USceneComponent> Head;

	// FUNCTIONS---------------------------------

	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	void InitializeFixture(UStaticMeshComponent* StaticMeshLens, UStaticMeshComponent* StaticMeshBeam);

	/** Pushes DMX Values to the Fixture. Expects normalized values in the range of 0.0f - 1.0f */
	virtual void PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttributeMap) override;
	
public:
	/** 
	 * Updates the spotlight intensity. 
	 * Considers light intensity max, the spotlight intensity scale and the cone angle to compute the intensity. 
	 * Should be used instead of setting the intensity of the spotlight directly.
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	void UpdateSpotLightIntensity();

	/** Sets the a new max light intensity */
	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	void SetLightIntensityMax(float NewLightIntensityMax);

	/** Sets a new max light distance */
	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	void SetLightDistanceMax(float NewLightDistanceMax);

	/** Sets a new light color temperature */
	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	void SetLightColorTemp(float NewLightColorTemp);

	/** Sets a new spotlight intensity scale */
	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	void SetSpotlightIntensityScale(float NewSpotlightIntensityScale);

	/** Sets a new pointlight intensity scale */
	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	void SetPointlightIntensityScale(float NewPointlightIntensityScale);

	/** Sets if the light should cast shadows */
	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	void SetLightCastShadow(bool bLightShouldCastShadow);

	// PARAMETERS---------------------------------

	// Light intensity at 1 steradian (32.77deg half angle)
	UPROPERTY(EditAnywhere, BlueprintSetter = SetLightIntensityMax, Category = "DMX Light Fixture")
	float LightIntensityMax;

	// Sets Attenuation Radius on the spotlight and pointlight
	UPROPERTY(EditAnywhere, BlueprintSetter = SetLightDistanceMax, Category = "DMX Light Fixture")
	float LightDistanceMax;

	// Light color temperature on the spotlight and pointlight
	UPROPERTY(EditAnywhere, BlueprintSetter = SetLightColorTemp, Category = "DMX Light Fixture")
	float LightColorTemp;

	// Scales spotlight intensity
	UPROPERTY(EditAnywhere, BlueprintSetter = SetSpotlightIntensityScale, Category = "DMX Light Fixture")
	float SpotlightIntensityScale;

	// Scales pointlight intensity
	UPROPERTY(EditAnywhere, BlueprintSetter = SetPointlightIntensityScale, Category = "DMX Light Fixture")
	float PointlightIntensityScale;

	// Enable/disable cast shadow on the spotlight and pointlight
	UPROPERTY(EditAnywhere, BlueprintSetter = SetLightCastShadow, Category = "DMX Light Fixture")
	bool LightCastShadow;

	// Simple solution useful for walls, 1 linetrace from the center
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX Light Fixture")
	bool UseDynamicOcclusion;

	// Disable lights rendering in the fixture to save on GPU
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DMX Light Fixture")
	bool DisableLights;

	// COMPONENTS ---------------------------------

	UPROPERTY(BlueprintReadOnly, Category = "DMX Light Fixture")
	TArray<TObjectPtr<UStaticMeshComponent>> StaticMeshComponents;
	
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<class USpotLightComponent> SpotLight;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<class UPointLightComponent> PointLight;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<class UArrowComponent> OcclusionDirection;


	// MATERIALS ---------------------------------

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "DMX Light Fixture")
	TObjectPtr<UMaterialInstance> LensMaterialInstance;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "DMX Light Fixture")
	TObjectPtr<UMaterialInstance> BeamMaterialInstance;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "DMX Light Fixture")
	TObjectPtr<UMaterialInstance> SpotLightMaterialInstance;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "DMX Light Fixture")
	TObjectPtr<UMaterialInstance> PointLightMaterialInstance;

	UPROPERTY(BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<UMaterialInstanceDynamic> DynamicMaterialLens;

	UPROPERTY(BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<UMaterialInstanceDynamic> DynamicMaterialBeam;

	UPROPERTY(BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<UMaterialInstanceDynamic> DynamicMaterialSpotLight;

	UPROPERTY(BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<UMaterialInstanceDynamic> DynamicMaterialPointLight;

	///////////////////////////////////////
	// DEPRECATED 5.1
public:
	UE_DEPRECATED(5.1, "Use BeamQuality instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use BeamQuality instead."))
	float MinQuality_DEPRECATED;

	UE_DEPRECATED(5.1, "Use ZoomQuality instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use ZoomQuality instead."))
	float MaxQuality_DEPRECATED;
};
