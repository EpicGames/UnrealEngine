// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "DMXFixtureActorBase.generated.h"

struct FDMXNormalizedAttributeValueMap;
class UDMXComponent;


UCLASS()
class DMXFIXTURES_API ADMXFixtureActorBase
	: public AActor
{
	GENERATED_BODY()

public:
	ADMXFixtureActorBase();

	/** Pushes DMX Values to the Fixture. Expects normalized values in the range of 0.0f - 1.0f */
	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	virtual void PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttributeMap);

	UFUNCTION(BlueprintCallable, Category = "DMX Fixture")
	void InterpolateDMXComponents(float DeltaSeconds);

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "DMX Light Fixture")
	TObjectPtr<UDMXComponent> DMX;
};
