// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXFixtureActor.h"

#include "ProceduralMeshComponent.h"
#include "RHI.h"
#include "Rendering/Texture2DResource.h"

#include "DMXFixtureActorMatrix.generated.h"


UCLASS()
class DMXFIXTURES_API ADMXFixtureActorMatrix : public ADMXFixtureActor
{
	GENERATED_BODY()

public:
	ADMXFixtureActorMatrix();

protected:
	
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:
	//~ Begin DMXGDTFActorInterface interface
	virtual void OnMVRGetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames, TArray<FName>& OutMatrixAttributeNames) const override;
	//~ End DMXGDTFActorInterface interface

	TArray<uint8> MatrixData;
	int MatrixDataSize;
	int NbrTextureRows;
	int QuadIndexCount;
	int XCells;
	int YCells;

	UTexture2D* MatrixDataTexture;
	FUpdateTextureRegion2D TextureRegion;

	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "DMX Matrix Fixture")
	TObjectPtr<UProceduralMeshComponent> MatrixHead;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Matrix Fixture")
	float MatrixWidth;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Matrix Fixture")
	float MatrixHeight;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Matrix Fixture")
	float MatrixDepth;

	/** If set to true, ignores the distribution set in the patch (hence the distribution becomes visible when not left to right, top to bottom) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, AdvancedDisplay, Category = "DMX Matrix Fixture")
	bool bIgnorePixelMappingDistributionOfFixturePatch = false;

	void GenerateMatrixMesh();
	void GenerateMatrixCells();
	void GenerateMatrixBeam();
	void GenerateMatrixChassis(FVector TL, FVector BL, FVector BR, FVector TR);
	void AddQuad(FVector TL, FVector BL, FVector BR, FVector TR, FProcMeshTangent Tangent);
	void UpdateDynamicTexture();
	FLinearColor GetMatrixAverageColor();
	void UpdateMatrixData(int32 RowIndex, int32 CellIndex, int32 ChannelIndex, float Value);

	UFUNCTION(BlueprintCallable, Category = "DMX Matrix Fixture")
	void PushFixtureMatrixCellData(TArray<FDMXCell> MatrixPixelData);

	UFUNCTION(BlueprintCallable, Category = "DMX Matrix Fixture")
	void InitializeMatrixFixture();

	UFUNCTION(BlueprintCallable, Category = "DMX Matrix Fixture")
	void GenerateEditorMatrixMesh();

protected:
	/** Sets the matrix fixture in a defaulted state using default values of its Fixture Components */
	void SetDefaultMatrixFixtureState();

private:
	TArray<FVector> Vertices;
	TArray<int32> Triangles;
	TArray<FVector> Normals;
	TArray<FProcMeshTangent> Tangents;
	TArray<FVector2D> UV0;
	TArray<FVector2D> UV1;
	TArray<FVector2D> UV2;
	TArray<FColor> Colors;
};
