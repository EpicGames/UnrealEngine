// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "Components/SpotLightComponent.h"
#include "Engine/DataTable.h"
#include "UObject/NameTypes.h"
#include "DMXProtocolTypes.h"
#include "DMXAttribute.h"
#include "DMXInterpolation.h"
#include "DMXFixtureComponent.generated.h"

struct FDMXNormalizedAttributeValueMap;


USTRUCT(BlueprintType)
struct FDMXChannelData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Channel")
	FDMXAttributeName Name;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Channel")
	float MinValue = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Channel")
	float MaxValue = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Channel")
	float DefaultValue = 0.0f;
};


UCLASS(Abstract, Meta=(IsBlueprintBase=false))
class DMXFIXTURES_API UDMXFixtureComponent : public UActorComponent
{
	GENERATED_BODY()

protected:
	/** Initializes the cells for the fixture */
	void InitCells(int NumCells);

public:
	UDMXFixtureComponent();

	/** Initializes the component */
	UFUNCTION()
	virtual void Initialize();

	/** Sets the cell that is currently active */
	virtual void SetCurrentCell(int Index);

	/** Pushes DMX Values to the Fixture Component. Expects normalized values in the range of 0.0f to 1.0f */
	virtual void PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttribute)  PURE_VIRTUAL(UDMXFixtureComponent::PushNormalizedValuesPerAttribute, return; );

	/** If used within a DMX Fixture Actor or Fixture Matrix Actor, the component only receives data when set to true. Else needs be implemented in blueprints. */
	UPROPERTY(EditAnywhere, Category = "DMX Parameters", meta = (DisplayPriority = 0))
	bool bIsEnabled = true;

	/** Value changes smaller than this threshold are ignored */
	UPROPERTY(EditAnywhere, Category = "DMX Parameters")
	float SkipThreshold = 0.01f;

	/** If used within a DMX Fixture Actor or Fixture Matrix Actor, the plugin interpolates towards the last set value. */
	UPROPERTY(EditAnywhere, Category = "DMX Parameters")
	bool bUseInterpolation = false;

	/** The scale of the interpolation speed. Faster when > 1, slower when < 1 */
	UPROPERTY(EditAnywhere, Category = "DMX Parameters")
	float InterpolationScale = 1.0f;

	/** True if the component is attached to a matrix fixture */
	UPROPERTY()
	bool bUsingMatrixData = false;

	/** If attached to a DMX Fixture Actor, returns the parent fixture actor. */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	class ADMXFixtureActor* GetParentFixtureActor();

	/** 
	 * Reads pixel color in the middle of each "Texture" and output linear colors. 
	 * Note, this function locks the GPU to read the color and is slow to call, even if Update TextureResource is not enabled.
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	TArray<FLinearColor> GetTextureCenterColors(UTexture2D* TextureAtlas, int numTextures, bool bUpdateTextureResource = true);

	/** Called each tick when interpolation is enabled, to calculate the next value */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "DMX")
	void InterpolateComponent(float DeltaSeconds);

	/** Called to initialize the component in blueprints */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "DMX")
	void InitializeComponent();

	/** Should be implemented to let other objects (e.g. datasmith) know which attributes the component can handle */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "DMX")
	void GetSupportedDMXAttributes(TArray<FName>& OutAttributeNames);
	virtual void GetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames) {};

	/** Applies the speed scale property */
	void ApplySpeedScale();

	// A cell represent one "lens" in a light fixture
	// i.e.: Single light fixture contains one cell but Matrix fixtures contain multiple cells
	// Also, a cell can have multiple channels (single, double)
	TArray<FCell> Cells;

	/** The currently handled cell */
	FCell* CurrentCell;

	// DERECATED 4.27
public:
	UE_DEPRECATED(4.27, "Removed, was unused 4.26 and has no clear use.")
	virtual void SetBitResolution(TMap<FDMXAttributeName, EDMXFixtureSignalFormat> AttributeNameToSignalFormatMap) {};
	
	UE_DEPRECATED(4.27, "Only call initialize instead.")
	virtual void SetRangeValue() {};
};
