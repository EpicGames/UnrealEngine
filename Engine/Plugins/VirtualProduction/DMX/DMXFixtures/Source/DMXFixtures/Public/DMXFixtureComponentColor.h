// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXFixtureComponent.h"
#include "DMXFixtureComponentColor.generated.h"

/**
 * Specific class to handle color mixing using 4 channels (rgb, cmy, rgbw).
 * Note, the color values are never interpolated.
 */
UCLASS(ClassGroup = DMXFixtureComponent, meta = (IsBlueprintBase = true))
class DMXFIXTURES_API UDMXFixtureComponentColor : public UDMXFixtureComponent
{
	GENERATED_BODY()
	
public:
	UDMXFixtureComponentColor();

	//~ Begin DMXFixtureComponent interface
	virtual void PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttribute) override;
	virtual void GetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames) override;
	//~ End DMXFixtureComponent interface

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX Channel")
	FDMXAttributeName DMXChannel1;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX Channel")
	FDMXAttributeName DMXChannel2;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX Channel")
	FDMXAttributeName DMXChannel3;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX Channel")
	FDMXAttributeName DMXChannel4;

	/** Initializes the interpolation range of the channels */
	virtual void Initialize() override;

	/** Sets the current cell */
	virtual void SetCurrentCell(int Index) override;

	/** True if the color is valid for the component */
	bool IsColorValid(const FLinearColor& NewColor) const;

	/** Sets the target color for the current cell */
	void SetTargetColor(const FLinearColor& NewColor);

	/** Sets the color of the component. Note DMX Fixture Component Color does not support interpolation */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "DMX")
	void SetColorNoInterp(const FLinearColor& NewColor);

	/** The target color, when interpolating. Otherwise just the final color */
	TArray<FLinearColor> TargetColorArray;

	/** Pointer to the current target color, corresponding to the current cell */
	FLinearColor* CurrentTargetColorRef;

	
	///////////////////////////////////////
	// DEPRECATED 4.27
public:	
	UE_DEPRECATED(4.27, "Replaced with SetColorNoInterp")
	void SetComponent(FLinearColor NewColor) { SetColorNoInterp(NewColor); }
};
