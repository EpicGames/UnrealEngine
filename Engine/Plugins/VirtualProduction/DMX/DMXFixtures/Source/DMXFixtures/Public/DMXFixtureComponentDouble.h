// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXFixtureComponent.h"
#include "DMXFixtureComponentDouble.generated.h"

// Component that uses 2 DMX channels
UCLASS(ClassGroup = FixtureComponent, meta=(IsBlueprintBase = true))
class DMXFIXTURES_API UDMXFixtureComponentDouble : public UDMXFixtureComponent
{
	GENERATED_BODY()

public:
	UDMXFixtureComponentDouble();

	//~ Begin DMXFixtureComponent interface
	virtual void PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttribute) override;
	virtual void GetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames) override;
	//~ End DMXFixtureComponent interface

	/** The first dmx attribute the component handles */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Channels")
	FDMXChannelData DMXChannel1;

	/** The second dmx attribute the component handles */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Channels")
	FDMXChannelData DMXChannel2;

	/** Initializes the interpolation range of the channels */
	virtual void Initialize() override;
	
	/** Gets the interpolation delta value (step) for this frame */
	UFUNCTION(BlueprintPure, Category = "DMX")
	float GetDMXInterpolatedStep(int32 ChannelIndex) const;

	/** Gets the current interpolated value */
	UFUNCTION(BlueprintPure, Category = "DMX")
	float GetDMXInterpolatedValue(int32 ChannelIndex) const;

	/** Gets the target value towards which the component interpolates */
	UFUNCTION(BlueprintPure, Category = "DMX")
	float GetDMXTargetValue(int32 ChannelIndex) const;

	/** True if the target value is reached and no interpolation is required */
	UFUNCTION(BlueprintPure, Category = "DMX")
	bool IsDMXInterpolationDone(int32 ChannelIndex) const;

	/** Maps the normalized value to the compoenent's value range */
	float NormalizedToAbsoluteValue(int32 ChannelIndex, float Alpha) const;

	/** Returns true, if the target value is valid */
	bool IsTargetValid(int32 ChannelIndex, float Target);

	/** Sets the target value for specified channel index. Interpolates to the value if bUseInterpolation is true. */
	void SetTargetValue(int32 ChannelIndex, float Value);

	/** Sets first value of the second channel. When interpolation is enabled this function should be called until the value is reached, else just once */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "DMX")
	void SetChannel1ValueNoInterp(float Channel1Value);

	/** Sets second value of the second channel. When interpolation is enabled this function should be called until the value is reached, else just once */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "DMX")
	void SetChannel2ValueNoInterp(float Channel2Value);



	///////////////////////////////////////
	// DEPRECATED 4.27
public:	
	UE_DEPRECATED(4.27, "Replaced with SetTargetValue (handling both Push and SetTarget).")
	void Push(int32 ChannelIndex, float Target) { SetTargetValue(ChannelIndex, Target); }

	UE_DEPRECATED(4.27, "Replaced with SetTargetValue (handling both Push and SetTarget).")
	void SetTarget(int32 ChannelIndex, float Target) { SetTargetValue(ChannelIndex, Target); }

	UE_DEPRECATED(4.27, "Replaced with GetDMXInterpolatedStep")
	float DMXInterpolatedStep(int32 ChannelIndex) { return GetDMXInterpolatedStep(ChannelIndex); }

	UE_DEPRECATED(4.27, "Replaced with GetDMXInterpolatedValue")
	float DMXInterpolatedValue(int32 ChannelIndex) { return GetDMXInterpolatedValue(ChannelIndex); }

	UE_DEPRECATED(4.27, "Replaced with GetDMXTargetValue")
	float DMXTargetValue(int32 ChannelIndex) { return GetDMXTargetValue(ChannelIndex); }

	UE_DEPRECATED(4.27, "Replaced with NormalizedToAbsoluteValue")
	float RemapValue(int32 ChannelIndex, float Alpha) { return NormalizedToAbsoluteValue(ChannelIndex, Alpha); }

	UE_DEPRECATED(4.27, "Replaced with SetChannel1ValueNoInterp")
	void SetComponentChannel1(float NewValue) { SetChannel1ValueNoInterp(NewValue); }

	UE_DEPRECATED(4.27, "Replaced with SetChannel1ValueNoInterp")
	void SetComponentChannel2(float NewValue) { SetChannel2ValueNoInterp(NewValue); }
};
