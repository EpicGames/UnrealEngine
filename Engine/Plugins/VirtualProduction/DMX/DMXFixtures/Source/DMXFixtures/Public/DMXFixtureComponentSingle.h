// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DMXFixtureComponent.h"
#include "DMXFixtureComponentSingle.generated.h"

// Component that uses 1 DMX channel
UCLASS(ClassGroup = FixtureComponent, meta=(IsBlueprintBase = true))
class DMXFIXTURES_API UDMXFixtureComponentSingle : public UDMXFixtureComponent
{
	GENERATED_BODY()

public:
	UDMXFixtureComponentSingle();

	//~ Begin DMXFixtureComponent interface
	virtual void PushNormalizedValuesPerAttribute(const FDMXNormalizedAttributeValueMap& ValuePerAttribute) override;
	virtual void GetSupportedDMXAttributes_Implementation(TArray<FName>& OutAttributeNames) override;
	//~ End DMXFixtureComponent interface
		
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DMX Channels")
	FDMXChannelData DMXChannel;

	/** Initializes the interpolation range of the channels */
	virtual void Initialize() override;

	/** Gets the interpolation delta value (step) for this frame */
	UFUNCTION(BlueprintPure, Category = "DMX")
	float GetDMXInterpolatedStep() const;

	/** Gets the current interpolated value */
	UFUNCTION(BlueprintPure, Category = "DMX")
	float GetDMXInterpolatedValue() const;

	/** Gets the target value towards which the component interpolates */
	UFUNCTION(BlueprintPure, Category = "DMX")
	float GetDMXTargetValue() const;

	/** True if the target value is reached and no interpolation is required */
	UFUNCTION(BlueprintPure, Category = "DMX")
	bool IsDMXInterpolationDone() const;
	
	/** Maps the normalized value to the compoenent's value range */
	float NormalizedToAbsoluteValue(float Alpha) const;

	/** Retuns true if the target differs from the previous target, and when interpolating, from the current value */
	bool IsTargetValid(float Target);

	/** Sets the target value. Interpolates to the value if bUseInterpolation is true. Expects the value to be in value range of the component */
	void SetTargetValue(float AbsoluteValue);
	
	/** Called to set the value. When interpolation is enabled this function is called by the plugin until the target value is reached, else just once. */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "DMX Component")
	void SetValueNoInterp(float NewValue);



	///////////////////////////////////////
	// DEPRECATED 4.27
public:	
	UE_DEPRECATED(4.27, "Replaced with SetTargetValue (handling both Push and SetTarget).")
	void Push(float Target) { SetTargetValue(Target); }

	UE_DEPRECATED(4.27, "Replaced with SetTargetValue (handling both Push and SetTarget).")
	void SetTarget(float Target) { SetTargetValue(Target); }

	UE_DEPRECATED(4.27, "Replaced with GetDMXInterpolatedStep")
	float DMXInterpolatedStep() { return GetDMXInterpolatedStep(); }

	UE_DEPRECATED(4.27, "Replaced with GetDMXInterpolatedValue")
	float DMXInterpolatedValue() { return GetDMXInterpolatedValue(); }

	UE_DEPRECATED(4.27, "Replaced with GetDMXTargetValue")
	float DMXTargetValue() { return GetDMXTargetValue(); }

	UE_DEPRECATED(4.27, "Replaced with NormalizedToAbsoluteValue")
	float RemapValue(float Alpha) { return NormalizedToAbsoluteValue(Alpha); }

	UE_DEPRECATED(4.27, "Replaced with SetValueNoInterp")
	void SetComponent(float NewValue) { SetValueNoInterp(NewValue); }
};
