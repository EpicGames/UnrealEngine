// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


class DMXFIXTURES_API FDMXFixturesModule : public IModuleInterface
{

public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	static FDMXFixturesModule& Get()
	{
		return FModuleManager::LoadModuleChecked< FDMXFixturesModule >("DMXFixtures");
	}

	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("DMXFixtures");
	}
};
