// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/KismetMathLibrary.h"

// Interpolation that provides a damping effect and support direction changes
struct FInterpolationData
{
	bool IsUpdating;
	float InterpolationScale;

	float CurrentValue;
	float TargetValue;
	float ToTravel;
	float RangeValue;
	float Direction;
	float TotalTravel;
	float PreviousT;
	float PreviousStep;
	
	float CurrentSpeed;
	float AccelerationThreshold;
	float SpeedIncMin;
	float SpeedIncMid;
	float SpeedIncMax;
	float SpeedMinimum;
	
	bool bFirstValueWasSet;

	FInterpolationData()
	{
		IsUpdating = false;
		InterpolationScale = 1.0f;
		RangeValue = 1.0f;
		TargetValue = 0.0f;
		CurrentValue = 0.0f;
		Direction = 0.0f;
		TotalTravel = 0.0f;
		ToTravel = 0.0f;
		PreviousT = 0.0f;
		PreviousStep = 0.0f;
		CurrentSpeed = 0.0f;
		SpeedMinimum = 50.0f;
		AccelerationThreshold = 0.5f;
		SpeedIncMin = 15.0f;
		SpeedIncMid = 20.0f;
		SpeedIncMax = 30.0f;

		bFirstValueWasSet = false;
	}

	bool IsTargetValid(float Value, float SkipThreshold) const
	{
		return FMath::Abs(TargetValue - Value) >= SkipThreshold;
	}

	void SetValueNoInterp(float NewValue)
	{
		CurrentValue = NewValue;
		TargetValue = NewValue;
	}

	bool IsInterpolationDone() const
	{
		return ToTravel < 0.05f;
	}

	void EndInterpolation()
	{
		CurrentValue = TargetValue;
		Direction = 0.0f;
		TotalTravel = 0.0f;
		ToTravel = 0.0f;
		PreviousT = 0.0f;
		PreviousStep = 0.0f;
		AccelerationThreshold = 0.5f;
		IsUpdating = false;
	}

	void StartTravel(float NewTarget)
	{
		IsUpdating = true;
		Direction = (TargetValue < NewTarget) ? 1.0f : -1.0f;
		float Travel = FMath::Abs(TargetValue - NewTarget);
		TotalTravel = Travel;
		ToTravel = Travel;
		CurrentSpeed = SpeedIncMid * InterpolationScale;
		TargetValue = NewTarget;
	}

	void UpdateTravel(float NewTarget)
	{
		float TravelDelta = (TargetValue - NewTarget) * Direction * -1.0f;
		if (CurrentValue * Direction < NewTarget * Direction)
		{
			// front
			TotalTravel = FMath::Max(TotalTravel + TravelDelta, 0.0f);
			ToTravel = FMath::Max(ToTravel + TravelDelta, 0.0f);
			TargetValue = NewTarget;

			// shift accel/decel threshold
			float CurrentT = UKismetMathLibrary::SafeDivide(TotalTravel - ToTravel, TotalTravel);
			CurrentT *= 1.1f;
			AccelerationThreshold = FMath::Clamp(CurrentT, 0.5f, 0.9f);
		}
		else
		{
			// back
			float Travel = FMath::Abs(NewTarget - CurrentValue);
			TotalTravel = Travel;
			ToTravel = Travel;
			Direction *= -1.0f;
			TargetValue = NewTarget;
			CurrentSpeed = 0.0f;
		}
	}

	// TODO: instead of using the derivative of the SmoothStep, use a sine wave
	void Travel(float DeltaSeconds)
	{
		TotalTravel = FMath::Max(TotalTravel, 0.0f);
		ToTravel = FMath::Max(ToTravel, 0.0f);

		float CurrentT = UKismetMathLibrary::SafeDivide(TotalTravel - ToTravel, TotalTravel);
		CurrentT = FMath::Clamp(CurrentT, 0.0f, 1.0f);

		float CurrentStep = FMath::SmoothStep(0.0f, 1.0f, CurrentT);
		float Derivative = UKismetMathLibrary::SafeDivide(CurrentStep - PreviousStep, CurrentT - PreviousT);
		float TravelAlpha = FMath::Clamp(TotalTravel / RangeValue, 0.0f, 1.0f);
		float A = FMath::Lerp(SpeedIncMin * InterpolationScale, SpeedIncMid * InterpolationScale, TravelAlpha);
		float SpeedInc = FMath::Lerp(A, SpeedIncMax * InterpolationScale, Derivative);

		if (CurrentT < AccelerationThreshold)
		{
			// accelerate
			CurrentSpeed = FMath::Max(CurrentSpeed + SpeedInc, 0.0f);
		}
		else
		{
			// decelerate
			CurrentSpeed = FMath::Max(CurrentSpeed - SpeedInc, SpeedMinimum * InterpolationScale);
		}

		// save
		PreviousStep = CurrentStep;
		PreviousT = CurrentT;

		// increment 
		CurrentValue = CurrentValue + (CurrentSpeed * DeltaSeconds * Direction);
		ToTravel = ToTravel - FMath::Abs(CurrentSpeed * DeltaSeconds);
	}

	void Push(float NewTarget)
	{
		if (IsUpdating)
		{
			UpdateTravel(NewTarget);
		}
		else
		{
			StartTravel(NewTarget);
		}
	}

};

struct FCell
{
	TArray<FInterpolationData> ChannelInterpolation;
};



